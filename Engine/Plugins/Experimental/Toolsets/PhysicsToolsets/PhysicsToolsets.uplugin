{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Physics Toolsets",
	"Description": "AI-callable toolsets for working with Unreal Engine physics.",
	"Category": "Other",
	"EditorOnly": true,
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"NoRedist": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "PhysicsToolsets",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		}
	]
}
