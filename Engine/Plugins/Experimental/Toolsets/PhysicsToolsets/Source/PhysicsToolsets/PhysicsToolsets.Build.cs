// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PhysicsToolsets : ModuleRules
{
	public PhysicsToolsets(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		IWYUSupport = IWYUSupport.Full;
		bUseUnity = true;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"CoreUObject",
				"Engine",
				"PhysicsUtilities",
				"ToolsetRegistry",
				"UnrealEd",
			}
		);
	}
}
