// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsToolsets/Module.h"

#include "Modules/ModuleManager.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

#include "PhysicsToolsets/PhysicsAssetToolset.h"

#define LOCTEXT_NAMESPACE "FPhysicsToolsetsModule"

void FPhysicsToolsetsModule::StartupModule()
{
	UToolsetRegistry::RegisterToolsetClass(UPhysicsAssetToolset::StaticClass());
}

void FPhysicsToolsetsModule::ShutdownModule()
{
	UToolsetRegistry::UnregisterToolsetClass(UPhysicsAssetToolset::StaticClass());
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FPhysicsToolsetsModule, PhysicsToolsets);
DEFINE_LOG_CATEGORY(LogPhysicsToolsets);
