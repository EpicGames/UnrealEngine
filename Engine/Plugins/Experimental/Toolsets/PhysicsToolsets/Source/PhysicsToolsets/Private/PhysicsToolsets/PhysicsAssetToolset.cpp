// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsToolsets/PhysicsAssetToolset.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Chaos/ChaosEngineInterface.h"
#include "Engine/SkeletalMesh.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Misc/PackageName.h"
#include "PhysicsAssetUtils.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetToolset)

// ---------------------------------------------------------------------------
// File-local helpers
// ---------------------------------------------------------------------------

namespace
{
	EAngularConstraintMotion ToAngularMotion(EConstraintMotion Motion)
	{
		switch (Motion)
		{
		case EConstraintMotion::Limited: return ACM_Limited;
		case EConstraintMotion::Locked:  return ACM_Locked;
		default:                         return ACM_Free;
		}
	}

	EConstraintMotion FromAngularMotion(EAngularConstraintMotion Motion)
	{
		switch (Motion)
		{
		case ACM_Limited: return EConstraintMotion::Limited;
		case ACM_Locked:  return EConstraintMotion::Locked;
		default:          return EConstraintMotion::Free;
		}
	}
} // namespace

// ---------------------------------------------------------------------------
// Public API — shape primitives
// ---------------------------------------------------------------------------

UPhysicsAsset* UPhysicsAssetToolset::CreateFromMesh(
	const FString& MeshPath, bool bAssignToMesh)
{
	if (MeshPath.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("MeshPath cannot be empty."));
		return nullptr;
	}

	USkeletalMesh* Mesh = Cast<USkeletalMesh>(
		StaticLoadObject(USkeletalMesh::StaticClass(), nullptr, *MeshPath));
	if (!Mesh)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("No skeletal mesh found at '%s'."), *MeshPath));
		return nullptr;
	}

	const FString PackageDir = FPackageName::GetLongPackagePath(MeshPath);
	const FString AssetName = FPackageName::GetShortName(MeshPath) + TEXT("_PhysicsAsset");
	const FString NewPackagePath = PackageDir / AssetName;

	UPackage* NewPackage = CreatePackage(*NewPackagePath);
	if (!NewPackage)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Failed to create package at '%s'."), *NewPackagePath));
		return nullptr;
	}

	UPhysicsAsset* PhysicsAsset = NewObject<UPhysicsAsset>(
		NewPackage, FName(*AssetName), RF_Public | RF_Standalone | RF_Transactional);

	FPhysAssetCreateParams Params;
	FText ErrorMessage;
	const bool bSuccess = FPhysicsAssetUtils::CreateFromSkeletalMesh(
		PhysicsAsset, Mesh, Params, ErrorMessage, bAssignToMesh, /*bShowProgress=*/false);

	if (!bSuccess)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Failed to create physics asset for '%s': %s"),
				*MeshPath, *ErrorMessage.ToString()));
		return nullptr;
	}

	FAssetRegistryModule::AssetCreated(PhysicsAsset);
	PhysicsAsset->MarkPackageDirty();
	return PhysicsAsset;
}

TArray<FString> UPhysicsAssetToolset::GetBodyNames(UPhysicsAsset* PhysicsAsset)
{
	if (!PhysicsAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("PhysicsAsset is null."));
		return {};
	}

	TArray<FString> Names;
	Names.Reserve(PhysicsAsset->SkeletalBodySetups.Num());
	for (const TObjectPtr<USkeletalBodySetup>& Body : PhysicsAsset->SkeletalBodySetups)
	{
		if (Body)
		{
			Names.Add(Body->BoneName.ToString());
		}
	}
	return Names;
}

TArray<FPhysicsShapeInfo> UPhysicsAssetToolset::GetBodyShapes(
	UPhysicsAsset* PhysicsAsset, const FString& BoneName)
{
	if (!PhysicsAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("PhysicsAsset is null."));
		return {};
	}

	USkeletalBodySetup* Body = FindBody(PhysicsAsset, BoneName);
	if (!Body)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("No body found for bone '%s'."), *BoneName));
		return {};
	}

	TArray<FPhysicsShapeInfo> Shapes;

	for (const FKSphereElem& Elem : Body->AggGeom.SphereElems)
	{
		FPhysicsShapeInfo& Info = Shapes.AddDefaulted_GetRef();
		Info.ShapeName = Elem.GetName().ToString();
		Info.ShapeType = EPhysicsShapeType::Sphere;
		Info.Center = Elem.Center;
		Info.Radius = Elem.Radius;
	}

	for (const FKSphylElem& Elem : Body->AggGeom.SphylElems)
	{
		FPhysicsShapeInfo& Info = Shapes.AddDefaulted_GetRef();
		Info.ShapeName = Elem.GetName().ToString();
		Info.ShapeType = EPhysicsShapeType::Capsule;
		Info.Center = Elem.Center;
		Info.Rotation = Elem.Rotation;
		Info.Radius = Elem.Radius;
		Info.Length = Elem.Length;
	}

	for (const FKBoxElem& Elem : Body->AggGeom.BoxElems)
	{
		FPhysicsShapeInfo& Info = Shapes.AddDefaulted_GetRef();
		Info.ShapeName = Elem.GetName().ToString();
		Info.ShapeType = EPhysicsShapeType::Box;
		Info.Center = Elem.Center;
		Info.Rotation = Elem.Rotation;
		Info.ExtentX = Elem.X;
		Info.ExtentY = Elem.Y;
		Info.ExtentZ = Elem.Z;
	}

	return Shapes;
}

void UPhysicsAssetToolset::SetSphere(
	UPhysicsAsset* PhysicsAsset, const FString& BoneName,
	const FString& ShapeName, const FVector& Center, float Radius)
{
	USkeletalBodySetup* Body = FindBodyForShape(PhysicsAsset, BoneName, ShapeName);
	if (!Body) return;
	if (Radius <= 0.f)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Radius must be greater than zero."));
		return;
	}

	const FName Name = BeginShapeEdit(PhysicsAsset, Body, ShapeName);

	FKSphereElem& NewElem = Body->AggGeom.SphereElems.AddDefaulted_GetRef();
	NewElem.SetName(Name);
	NewElem.Center = Center;
	NewElem.Radius = Radius;

	NotifyAssetChanged(PhysicsAsset);
}

void UPhysicsAssetToolset::SetCapsule(
	UPhysicsAsset* PhysicsAsset, const FString& BoneName,
	const FString& ShapeName, const FVector& Center, const FRotator& Rotation,
	float Radius, float Length)
{
	USkeletalBodySetup* Body = FindBodyForShape(PhysicsAsset, BoneName, ShapeName);
	if (!Body) return;
	if (Radius <= 0.f)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Radius must be greater than zero."));
		return;
	}
	if (Length < 0.f)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Length must be non-negative."));
		return;
	}

	const FName Name = BeginShapeEdit(PhysicsAsset, Body, ShapeName);

	FKSphylElem& NewElem = Body->AggGeom.SphylElems.AddDefaulted_GetRef();
	NewElem.SetName(Name);
	NewElem.Center = Center;
	NewElem.Rotation = Rotation;
	NewElem.Radius = Radius;
	NewElem.Length = Length;

	NotifyAssetChanged(PhysicsAsset);
}

void UPhysicsAssetToolset::SetBox(
	UPhysicsAsset* PhysicsAsset, const FString& BoneName,
	const FString& ShapeName, const FVector& Center, const FRotator& Rotation,
	float ExtentX, float ExtentY, float ExtentZ)
{
	USkeletalBodySetup* Body = FindBodyForShape(PhysicsAsset, BoneName, ShapeName);
	if (!Body) return;
	if (ExtentX <= 0.f || ExtentY <= 0.f || ExtentZ <= 0.f)
	{
		UKismetSystemLibrary::RaiseScriptError(
			TEXT("ExtentX, ExtentY, and ExtentZ must all be greater than zero."));
		return;
	}

	const FName Name = BeginShapeEdit(PhysicsAsset, Body, ShapeName);

	FKBoxElem& NewElem = Body->AggGeom.BoxElems.AddDefaulted_GetRef();
	NewElem.SetName(Name);
	NewElem.Center = Center;
	NewElem.Rotation = Rotation;
	NewElem.X = ExtentX;
	NewElem.Y = ExtentY;
	NewElem.Z = ExtentZ;

	NotifyAssetChanged(PhysicsAsset);
}

void UPhysicsAssetToolset::RemoveShape(
	UPhysicsAsset* PhysicsAsset, const FString& BoneName, const FString& ShapeName)
{
	USkeletalBodySetup* Body = FindBodyForShape(PhysicsAsset, BoneName, ShapeName);
	if (!Body) return;

	const int32 Before =
		Body->AggGeom.SphereElems.Num() +
		Body->AggGeom.SphylElems.Num() +
		Body->AggGeom.BoxElems.Num();

	const FName Name(*ShapeName);
	PhysicsAsset->Modify();
	Body->Modify();
	RemoveShapeByName(Body, Name);

	const int32 After =
		Body->AggGeom.SphereElems.Num() +
		Body->AggGeom.SphylElems.Num() +
		Body->AggGeom.BoxElems.Num();

	if (Before == After)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(
				TEXT("No shape named '%s' found on body '%s'."), *ShapeName, *BoneName));
		return;
	}

	NotifyAssetChanged(PhysicsAsset);
}

// ---------------------------------------------------------------------------
// Public API — body CRUD
// ---------------------------------------------------------------------------

void UPhysicsAssetToolset::AddBody(UPhysicsAsset* PhysicsAsset, const FString& BoneName)
{
	if (!PhysicsAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("PhysicsAsset is null."));
		return;
	}
	if (BoneName.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("BoneName cannot be empty."));
		return;
	}
	if (FindBody(PhysicsAsset, BoneName))
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("A body for bone '%s' already exists."), *BoneName));
		return;
	}

	PhysicsAsset->Modify();

	USkeletalBodySetup* NewBody = NewObject<USkeletalBodySetup>(PhysicsAsset);
	NewBody->BoneName = FName(*BoneName);
	PhysicsAsset->SkeletalBodySetups.Add(NewBody);

	NotifyAssetChanged(PhysicsAsset);
}

void UPhysicsAssetToolset::RemoveBody(UPhysicsAsset* PhysicsAsset, const FString& BoneName)
{
	if (!PhysicsAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("PhysicsAsset is null."));
		return;
	}

	const int32 BodyIndex = PhysicsAsset->FindBodyIndex(FName(*BoneName));
	if (BodyIndex == INDEX_NONE)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("No body found for bone '%s'."), *BoneName));
		return;
	}

	PhysicsAsset->Modify();

	// Remove all constraints that reference this body (highest index first).
	TArray<int32> ConstraintIndices;
	PhysicsAsset->BodyFindConstraints(BodyIndex, ConstraintIndices);
	ConstraintIndices.Sort([](int32 A, int32 B) { return A > B; });
	for (const int32 Idx : ConstraintIndices)
	{
		PhysicsAsset->ConstraintSetup[Idx]->Modify();
		PhysicsAsset->ConstraintSetup.RemoveAt(Idx);
	}

	PhysicsAsset->SkeletalBodySetups[BodyIndex]->Modify();
	PhysicsAsset->SkeletalBodySetups.RemoveAt(BodyIndex);

	NotifyAssetChanged(PhysicsAsset);
}

// ---------------------------------------------------------------------------
// Public API — body properties
// ---------------------------------------------------------------------------

EBodyPhysicsMode UPhysicsAssetToolset::GetBodyPhysicsMode(
	UPhysicsAsset* PhysicsAsset, const FString& BoneName)
{
	if (!PhysicsAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("PhysicsAsset is null."));
		return EBodyPhysicsMode::Default;
	}

	USkeletalBodySetup* Body = FindBody(PhysicsAsset, BoneName);
	if (!Body)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("No body found for bone '%s'."), *BoneName));
		return EBodyPhysicsMode::Default;
	}

	switch (Body->PhysicsType)
	{
	case PhysType_Kinematic: return EBodyPhysicsMode::Kinematic;
	case PhysType_Simulated: return EBodyPhysicsMode::Simulated;
	default:                 return EBodyPhysicsMode::Default;
	}
}

void UPhysicsAssetToolset::SetBodyPhysicsMode(
	UPhysicsAsset* PhysicsAsset, const FString& BoneName, EBodyPhysicsMode Mode)
{
	if (!PhysicsAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("PhysicsAsset is null."));
		return;
	}

	USkeletalBodySetup* Body = FindBody(PhysicsAsset, BoneName);
	if (!Body)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("No body found for bone '%s'."), *BoneName));
		return;
	}

	PhysicsAsset->Modify();
	Body->Modify();

	switch (Mode)
	{
	case EBodyPhysicsMode::Kinematic: Body->PhysicsType = PhysType_Kinematic; break;
	case EBodyPhysicsMode::Simulated: Body->PhysicsType = PhysType_Simulated; break;
	default:                          Body->PhysicsType = PhysType_Default;   break;
	}

	NotifyAssetChanged(PhysicsAsset);
}

float UPhysicsAssetToolset::GetBodyMassScale(
	UPhysicsAsset* PhysicsAsset, const FString& BoneName)
{
	if (!PhysicsAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("PhysicsAsset is null."));
		return -1.f;
	}

	USkeletalBodySetup* Body = FindBody(PhysicsAsset, BoneName);
	if (!Body)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("No body found for bone '%s'."), *BoneName));
		return -1.f;
	}

	return Body->DefaultInstance.MassScale;
}

void UPhysicsAssetToolset::SetBodyMassScale(
	UPhysicsAsset* PhysicsAsset, const FString& BoneName, float MassScale)
{
	if (!PhysicsAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("PhysicsAsset is null."));
		return;
	}
	if (MassScale <= 0.f)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("MassScale must be greater than zero."));
		return;
	}

	USkeletalBodySetup* Body = FindBody(PhysicsAsset, BoneName);
	if (!Body)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("No body found for bone '%s'."), *BoneName));
		return;
	}

	PhysicsAsset->Modify();
	Body->Modify();
	Body->DefaultInstance.MassScale = MassScale;

	NotifyAssetChanged(PhysicsAsset);
}

// ---------------------------------------------------------------------------
// Public API — constraint CRUD
// ---------------------------------------------------------------------------

TArray<FPhysicsConstraintInfo> UPhysicsAssetToolset::GetConstraints(
	UPhysicsAsset* PhysicsAsset)
{
	if (!PhysicsAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("PhysicsAsset is null."));
		return {};
	}

	TArray<FPhysicsConstraintInfo> Result;
	Result.Reserve(PhysicsAsset->ConstraintSetup.Num());

	for (const TObjectPtr<UPhysicsConstraintTemplate>& CT : PhysicsAsset->ConstraintSetup)
	{
		if (!CT)
		{
			continue;
		}
		const FConstraintInstance& CI = CT->DefaultInstance;

		FPhysicsConstraintInfo& Info = Result.AddDefaulted_GetRef();
		Info.Bone1Name          = CI.ConstraintBone1.ToString();
		Info.Bone2Name          = CI.ConstraintBone2.ToString();
		Info.Swing1Motion       = FromAngularMotion(CI.GetAngularSwing1Motion());
		Info.Swing1LimitDegrees = CI.GetAngularSwing1Limit();
		Info.Swing2Motion       = FromAngularMotion(CI.GetAngularSwing2Motion());
		Info.Swing2LimitDegrees = CI.GetAngularSwing2Limit();
		Info.TwistMotion        = FromAngularMotion(CI.GetAngularTwistMotion());
		Info.TwistLimitDegrees  = CI.GetAngularTwistLimit();
	}

	return Result;
}

void UPhysicsAssetToolset::AddConstraint(
	UPhysicsAsset* PhysicsAsset, const FString& Bone1Name, const FString& Bone2Name)
{
	if (!PhysicsAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("PhysicsAsset is null."));
		return;
	}
	if (Bone1Name.IsEmpty() || Bone2Name.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Bone names cannot be empty."));
		return;
	}
	if (!FindBody(PhysicsAsset, Bone1Name))
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("No body found for bone '%s'."), *Bone1Name));
		return;
	}
	if (!FindBody(PhysicsAsset, Bone2Name))
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("No body found for bone '%s'."), *Bone2Name));
		return;
	}
	if (FindConstraintTemplate(PhysicsAsset, Bone1Name, Bone2Name))
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(
				TEXT("A constraint between '%s' and '%s' already exists."),
				*Bone1Name, *Bone2Name));
		return;
	}

	PhysicsAsset->Modify();

	UPhysicsConstraintTemplate* CT = NewObject<UPhysicsConstraintTemplate>(PhysicsAsset);
	CT->DefaultInstance.ConstraintBone1 = FName(*Bone1Name);
	CT->DefaultInstance.ConstraintBone2 = FName(*Bone2Name);
	PhysicsAsset->ConstraintSetup.Add(CT);

	NotifyAssetChanged(PhysicsAsset);
}

void UPhysicsAssetToolset::SetConstraintLimits(
	UPhysicsAsset* PhysicsAsset, FPhysicsConstraintInfo Info)
{
	if (!PhysicsAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("PhysicsAsset is null."));
		return;
	}

	UPhysicsConstraintTemplate* CT =
		FindConstraintTemplate(PhysicsAsset, Info.Bone1Name, Info.Bone2Name);
	if (!CT)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(
				TEXT("No constraint found between '%s' and '%s'."),
				*Info.Bone1Name, *Info.Bone2Name));
		return;
	}

	PhysicsAsset->Modify();
	CT->Modify();

	FConstraintInstance& CI = CT->DefaultInstance;
	CI.SetAngularSwing1Limit(ToAngularMotion(Info.Swing1Motion), Info.Swing1LimitDegrees);
	CI.SetAngularSwing2Limit(ToAngularMotion(Info.Swing2Motion), Info.Swing2LimitDegrees);
	CI.SetAngularTwistLimit(ToAngularMotion(Info.TwistMotion),   Info.TwistLimitDegrees);

	NotifyAssetChanged(PhysicsAsset);
}

void UPhysicsAssetToolset::RemoveConstraint(
	UPhysicsAsset* PhysicsAsset, const FString& Bone1Name, const FString& Bone2Name)
{
	if (!PhysicsAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("PhysicsAsset is null."));
		return;
	}

	const FName Name1(*Bone1Name);
	const FName Name2(*Bone2Name);

	int32 Index = PhysicsAsset->FindConstraintIndex(Name1, Name2);
	if (Index == INDEX_NONE)
	{
		Index = PhysicsAsset->FindConstraintIndex(Name2, Name1);
	}
	if (Index == INDEX_NONE)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(
				TEXT("No constraint found between '%s' and '%s'."),
				*Bone1Name, *Bone2Name));
		return;
	}

	PhysicsAsset->Modify();
	PhysicsAsset->ConstraintSetup[Index]->Modify();
	PhysicsAsset->ConstraintSetup.RemoveAt(Index);

	NotifyAssetChanged(PhysicsAsset);
}

// ---------------------------------------------------------------------------
// Private helpers
// ---------------------------------------------------------------------------

USkeletalBodySetup* UPhysicsAssetToolset::FindBody(
	UPhysicsAsset* PhysicsAsset, const FString& BoneName)
{
	const int32 Index = PhysicsAsset->FindBodyIndex(FName(*BoneName));
	if (Index == INDEX_NONE)
	{
		return nullptr;
	}
	return PhysicsAsset->SkeletalBodySetups[Index];
}

USkeletalBodySetup* UPhysicsAssetToolset::FindBodyForShape(
	UPhysicsAsset* PhysicsAsset, const FString& BoneName, const FString& ShapeName)
{
	if (!PhysicsAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("PhysicsAsset is null."));
		return nullptr;
	}
	if (ShapeName.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("ShapeName cannot be empty."));
		return nullptr;
	}
	USkeletalBodySetup* Body = FindBody(PhysicsAsset, BoneName);
	if (!Body)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("No body found for bone '%s'."), *BoneName));
		return nullptr;
	}
	return Body;
}

FName UPhysicsAssetToolset::BeginShapeEdit(
	UPhysicsAsset* PhysicsAsset, USkeletalBodySetup* Body, const FString& ShapeName)
{
	const FName Name(*ShapeName);
	PhysicsAsset->Modify();
	Body->Modify();
	RemoveShapeByName(Body, Name);
	return Name;
}

void UPhysicsAssetToolset::RemoveShapeByName(USkeletalBodySetup* Body, const FName& Name)
{
	Body->AggGeom.SphereElems.RemoveAll(
		[&Name](const FKSphereElem& E) { return E.GetName() == Name; });
	Body->AggGeom.SphylElems.RemoveAll(
		[&Name](const FKSphylElem& E) { return E.GetName() == Name; });
	Body->AggGeom.BoxElems.RemoveAll(
		[&Name](const FKBoxElem& E) { return E.GetName() == Name; });
}

void UPhysicsAssetToolset::NotifyAssetChanged(UPhysicsAsset* PhysicsAsset)
{
	PhysicsAsset->UpdateBoundsBodiesArray();
	PhysicsAsset->UpdateBodySetupIndexMap();
	PhysicsAsset->MarkPackageDirty();
#if WITH_EDITOR
	PhysicsAsset->InvalidateAllPhysicsMeshes();
	PhysicsAsset->RefreshPhysicsAssetChange();
#endif
}

UPhysicsConstraintTemplate* UPhysicsAssetToolset::FindConstraintTemplate(
	UPhysicsAsset* PhysicsAsset, const FString& Bone1Name, const FString& Bone2Name)
{
	const FName Name1(*Bone1Name);
	const FName Name2(*Bone2Name);

	int32 Index = PhysicsAsset->FindConstraintIndex(Name1, Name2);
	if (Index == INDEX_NONE)
	{
		Index = PhysicsAsset->FindConstraintIndex(Name2, Name1);
	}
	if (Index == INDEX_NONE)
	{
		return nullptr;
	}
	return PhysicsAsset->ConstraintSetup[Index];
}
