// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolsetRegistry/ToolsetDefinition.h"

#include "PhysicsAssetToolset.generated.h"

class UPhysicsAsset;
class UPhysicsConstraintTemplate;
class USkeletalBodySetup;

/** The type of collision primitive. */
UENUM(BlueprintType)
enum class EPhysicsShapeType : uint8
{
	Sphere,
	Capsule,
	Box,
};

/** Controls whether a physics body participates in world simulation. */
UENUM(BlueprintType)
enum class EBodyPhysicsMode : uint8
{
	/** Follows the skeletal mesh component's simulation state. */
	Default,
	/** Always kinematic: moves with animation and is not simulated. */
	Kinematic,
	/** Always simulated, regardless of the component's state. */
	Simulated,
};

/** How a constraint axis may move. */
UENUM(BlueprintType)
enum class EConstraintMotion : uint8
{
	/** No restriction — the axis is completely free. */
	Free,
	/** Restricted to the specified limit angle. */
	Limited,
	/** Fully locked — no movement on this axis. */
	Locked,
};

/**
 * Describes a single collision primitive on a physics body.
 * All positions and rotations are in the bone's local space.
 */
USTRUCT(BlueprintType)
struct FPhysicsShapeInfo
{
	GENERATED_BODY()

	/** User-defined name that uniquely identifies this shape on the body. */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsShape")
	FString ShapeName;

	/** The type of collision primitive. */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsShape")
	EPhysicsShapeType ShapeType = EPhysicsShapeType::Sphere;

	/** Center position in bone-local space (cm). */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsShape")
	FVector Center = FVector::ZeroVector;

	/**
	 * Orientation in bone-local space.
	 * For Capsule, Z is the long axis. Unused for Sphere.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsShape")
	FRotator Rotation = FRotator::ZeroRotator;

	/** Radius (cm). Used by Sphere and Capsule. */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsShape")
	float Radius = 0.f;

	/**
	 * Cylinder section length (cm). Used by Capsule only.
	 * Total capsule height = Length + 2 * Radius.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsShape")
	float Length = 0.f;

	/** Full extent along local X (cm). Used by Box only. */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsShape")
	float ExtentX = 0.f;

	/** Full extent along local Y (cm). Used by Box only. */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsShape")
	float ExtentY = 0.f;

	/** Full extent along local Z (cm). Used by Box only. */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsShape")
	float ExtentZ = 0.f;
};

/**
 * Describes the angular limits for a single constraint between two bodies.
 * Bone1 is the child bone; Bone2 is the parent bone.
 * All limit angles are in degrees [0, 180].
 */
USTRUCT(BlueprintType)
struct FPhysicsConstraintInfo
{
	GENERATED_BODY()

	/** Name of the child bone. */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsConstraint")
	FString Bone1Name;

	/** Name of the parent bone. */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsConstraint")
	FString Bone2Name;

	/** Motion type for Swing 1 (rotation around the body-local Y axis). */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsConstraint")
	EConstraintMotion Swing1Motion = EConstraintMotion::Free;

	/** Swing 1 limit in degrees. Used when Swing1Motion is Limited. */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsConstraint")
	float Swing1LimitDegrees = 45.f;

	/** Motion type for Swing 2 (rotation around the body-local Z axis). */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsConstraint")
	EConstraintMotion Swing2Motion = EConstraintMotion::Free;

	/** Swing 2 limit in degrees. Used when Swing2Motion is Limited. */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsConstraint")
	float Swing2LimitDegrees = 45.f;

	/** Motion type for Twist (rotation around the body-local X axis). */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsConstraint")
	EConstraintMotion TwistMotion = EConstraintMotion::Free;

	/** Twist limit in degrees. Used when TwistMotion is Limited. */
	UPROPERTY(BlueprintReadWrite, Category = "PhysicsConstraint")
	float TwistLimitDegrees = 45.f;
};

/** Provides tools for creating and managing Physics Assets. */
UCLASS(BlueprintType, MinimalAPI)
class UPhysicsAssetToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	/**
	 * Creates a physics asset from a skeletal mesh, auto-generating collision bodies for
	 * each bone. The asset is placed in the same folder as the mesh with the suffix
	 * "_PhysicsAsset".
	 * @param MeshPath Content-browser path to the skeletal mesh, e.g. '/Game/Characters/SKM_Hero'.
	 * @param bAssignToMesh If true, assigns the new physics asset to the mesh.
	 * @return The newly created physics asset.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static UPhysicsAsset* CreateFromMesh(const FString& MeshPath, bool bAssignToMesh);

	/**
	 * Returns the bone name for each rigid body in a physics asset.
	 * @param PhysicsAsset The physics asset to query.
	 * @return A list of bone names, one per body.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static TArray<FString> GetBodyNames(UPhysicsAsset* PhysicsAsset);

	/**
	 * Returns all collision shapes assigned to a body.
	 * @param PhysicsAsset The physics asset to query.
	 * @param BoneName The name of the bone whose body shapes to retrieve.
	 * @return A list of shape descriptors. Raises a script error if no body exists for the bone.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static TArray<FPhysicsShapeInfo> GetBodyShapes(
		UPhysicsAsset* PhysicsAsset, const FString& BoneName);

	/**
	 * Adds or replaces a sphere collision primitive on a body.
	 * If any shape with the given name already exists on the body it is removed first.
	 * @param PhysicsAsset The physics asset to modify.
	 * @param BoneName The name of the bone whose body to modify.
	 * @param ShapeName A name that uniquely identifies this shape on the body.
	 * @param Center Center of the sphere in bone-local space (cm).
	 * @param Radius Radius of the sphere (cm). Must be greater than zero.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static void SetSphere(
		UPhysicsAsset* PhysicsAsset, const FString& BoneName,
		const FString& ShapeName, const FVector& Center, float Radius);

	/**
	 * Adds or replaces a capsule collision primitive on a body.
	 * If any shape with the given name already exists on the body it is removed first.
	 * The capsule's long axis is its local Z after applying Rotation.
	 * @param PhysicsAsset The physics asset to modify.
	 * @param BoneName The name of the bone whose body to modify.
	 * @param ShapeName A name that uniquely identifies this shape on the body.
	 * @param Center Center of the capsule in bone-local space (cm).
	 * @param Rotation Orientation of the capsule in bone-local space.
	 * @param Radius Radius of the capsule end-caps (cm). Must be greater than zero.
	 * @param Length Length of the cylindrical section (cm). Must be non-negative.
	 *   Total capsule height = Length + 2 * Radius.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static void SetCapsule(
		UPhysicsAsset* PhysicsAsset, const FString& BoneName,
		const FString& ShapeName, const FVector& Center, const FRotator& Rotation,
		float Radius, float Length);

	/**
	 * Adds or replaces a box collision primitive on a body.
	 * If any shape with the given name already exists on the body it is removed first.
	 * @param PhysicsAsset The physics asset to modify.
	 * @param BoneName The name of the bone whose body to modify.
	 * @param ShapeName A name that uniquely identifies this shape on the body.
	 * @param Center Center of the box in bone-local space (cm).
	 * @param Rotation Orientation of the box in bone-local space.
	 * @param ExtentX Full extent along local X (cm). Must be greater than zero.
	 * @param ExtentY Full extent along local Y (cm). Must be greater than zero.
	 * @param ExtentZ Full extent along local Z (cm). Must be greater than zero.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static void SetBox(
		UPhysicsAsset* PhysicsAsset, const FString& BoneName,
		const FString& ShapeName, const FVector& Center, const FRotator& Rotation,
		float ExtentX, float ExtentY, float ExtentZ);

	/**
	 * Removes a collision primitive from a body by name.
	 * @param PhysicsAsset The physics asset to modify.
	 * @param BoneName The name of the bone whose body to modify.
	 * @param ShapeName The name of the shape to remove.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static void RemoveShape(
		UPhysicsAsset* PhysicsAsset, const FString& BoneName, const FString& ShapeName);

	// -------------------------------------------------------------------------
	// Body CRUD
	// -------------------------------------------------------------------------

	/**
	 * Adds a new empty body for the given bone.
	 * @param PhysicsAsset The physics asset to modify.
	 * @param BoneName The name of the bone to add a body for.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static void AddBody(UPhysicsAsset* PhysicsAsset, const FString& BoneName);

	/**
	 * Removes the body for the given bone along with any constraints that reference it.
	 * Raises a script error if PhysicsAsset is null or no body exists for BoneName.
	 * @param PhysicsAsset The physics asset to modify.
	 * @param BoneName The name of the bone whose body to remove.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static void RemoveBody(UPhysicsAsset* PhysicsAsset, const FString& BoneName);

	// -------------------------------------------------------------------------
	// Body properties
	// -------------------------------------------------------------------------

	/**
	 * Returns the physics simulation mode for the given body.
	 * @param PhysicsAsset The physics asset to query.
	 * @param BoneName The name of the bone whose body to query.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static EBodyPhysicsMode GetBodyPhysicsMode(
		UPhysicsAsset* PhysicsAsset, const FString& BoneName);

	/**
	 * Sets the physics simulation mode for the given body.
	 * @param PhysicsAsset The physics asset to modify.
	 * @param BoneName The name of the bone whose body to modify.
	 * @param Mode The desired simulation mode.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static void SetBodyPhysicsMode(
		UPhysicsAsset* PhysicsAsset, const FString& BoneName, EBodyPhysicsMode Mode);

	/**
	 * Returns the mass-scale multiplier for the given body.
	 * @param PhysicsAsset The physics asset to query.
	 * @param BoneName The name of the bone whose body to query.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static float GetBodyMassScale(UPhysicsAsset* PhysicsAsset, const FString& BoneName);

	/**
	 * Sets the mass-scale multiplier for the given body.
	 * @param PhysicsAsset The physics asset to modify.
	 * @param BoneName The name of the bone whose body to modify.
	 * @param MassScale Multiplier applied to the computed mass. Must be greater than zero.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static void SetBodyMassScale(
		UPhysicsAsset* PhysicsAsset, const FString& BoneName, float MassScale);

	// -------------------------------------------------------------------------
	// Constraint CRUD
	// -------------------------------------------------------------------------

	/**
	 * Returns all constraints in the physics asset with their current angular limits.
	 * @param PhysicsAsset The physics asset to query.
	 * @return A list of constraint descriptors.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static TArray<FPhysicsConstraintInfo> GetConstraints(UPhysicsAsset* PhysicsAsset);

	/**
	 * Adds a new constraint between two bodies. Both bodies must already exist.
	 * @param PhysicsAsset The physics asset to modify.
	 * @param Bone1Name Name of the child bone.
	 * @param Bone2Name Name of the parent bone.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static void AddConstraint(
		UPhysicsAsset* PhysicsAsset, const FString& Bone1Name, const FString& Bone2Name);

	/**
	 * Updates the angular limits for an existing constraint.
	 * @param PhysicsAsset The physics asset to modify.
	 * @param Info Constraint descriptor. Bone1Name and Bone2Name identify the constraint.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static void SetConstraintLimits(UPhysicsAsset* PhysicsAsset, FPhysicsConstraintInfo Info);

	/**
	 * Removes the constraint between two bodies.
	 * @param PhysicsAsset The physics asset to modify.
	 * @param Bone1Name Name of the child bone.
	 * @param Bone2Name Name of the parent bone.
	 */
	UFUNCTION(Category = "PhysicsAsset", meta = (AICallable))
	static void RemoveConstraint(
		UPhysicsAsset* PhysicsAsset, const FString& Bone1Name, const FString& Bone2Name);

private:
	/** Returns the body for BoneName, or nullptr if not found. */
	static USkeletalBodySetup* FindBody(UPhysicsAsset* PhysicsAsset, const FString& BoneName);

	/** Removes all shapes named Name from Body across all primitive types. */
	static void RemoveShapeByName(USkeletalBodySetup* Body, const FName& Name);

	/**
	 * Validates PhysicsAsset is non-null, ShapeName is non-empty, and a body exists for
	 * BoneName. Raises a script error and returns nullptr on the first failed check.
	 */
	static USkeletalBodySetup* FindBodyForShape(
		UPhysicsAsset* PhysicsAsset, const FString& BoneName, const FString& ShapeName);

	/**
	 * Marks PhysicsAsset and Body dirty, removes any existing shape named ShapeName,
	 * and returns the FName to assign to the new element.
	 */
	static FName BeginShapeEdit(
		UPhysicsAsset* PhysicsAsset, USkeletalBodySetup* Body, const FString& ShapeName);

	/** Marks the asset dirty and triggers editor refresh. */
	static void NotifyAssetChanged(UPhysicsAsset* PhysicsAsset);

	/**
	 * Returns the constraint template between the two bones, or nullptr if not found.
	 * Tries both orderings of the bone names.
	 */
	static UPhysicsConstraintTemplate* FindConstraintTemplate(
		UPhysicsAsset* PhysicsAsset, const FString& Bone1Name, const FString& Bone2Name);
};
