// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include "PhysicsToolsets/Tests/PhysicsToolsetsTestFlags.h"
#include "PhysicsToolsets/PhysicsAssetToolset.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FPhysicsAssetToolsetSpec, "AI.Toolsets.PhysicsToolsets.PhysicsAssetToolsetSpec",
	PhysicsToolsetsTest::Flags)

	// In-memory physics asset shared across tests.
	UPhysicsAsset* PhysicsAsset = nullptr;
	FString BoneName;
	FString BoneName2;

	/** Creates a minimal in-memory physics asset with a single body named "TestBone". */
	void SetUpInMemoryAsset()
	{
		PhysicsAsset = NewObject<UPhysicsAsset>(GetTransientPackage());
		USkeletalBodySetup* Body = NewObject<USkeletalBodySetup>(PhysicsAsset);
		Body->BoneName = FName("TestBone");
		PhysicsAsset->SkeletalBodySetups.Add(Body);
		PhysicsAsset->UpdateBodySetupIndexMap();
		BoneName = TEXT("TestBone");
	}

	/** Extends SetUpInMemoryAsset with a second body named "TestBone2". */
	void SetUpInMemoryAssetWithTwoBodies()
	{
		SetUpInMemoryAsset();
		USkeletalBodySetup* Body2 = NewObject<USkeletalBodySetup>(PhysicsAsset);
		Body2->BoneName = FName("TestBone2");
		PhysicsAsset->SkeletalBodySetups.Add(Body2);
		PhysicsAsset->UpdateBodySetupIndexMap();
		BoneName2 = TEXT("TestBone2");
	}

	void TearDownInMemoryAsset()
	{
		PhysicsAsset = nullptr;
		BoneName.Empty();
		BoneName2.Empty();
	}

END_DEFINE_SPEC(FPhysicsAssetToolsetSpec)

void FPhysicsAssetToolsetSpec::Define()
{
	// -------------------------------------------------------------------------
	// CreateFromMesh
	// -------------------------------------------------------------------------
	Describe(TEXT("CreateFromMesh"), [this]()
	{
		It(TEXT("Returns null for an empty mesh path"), [this]()
		{
			UPhysicsAsset* Result = UPhysicsAssetToolset::CreateFromMesh(FString(), false);
			TestNull("Null for empty path", Result);
		});

		It(TEXT("Returns null for an invalid mesh path"), [this]()
		{
			UPhysicsAsset* Result = UPhysicsAssetToolset::CreateFromMesh(
				TEXT("/Game/ZZZ_DoesNotExist/NonExistentMesh"), false);
			TestNull("Null for invalid path", Result);
		});

		It(TEXT("Creates a physics asset from a valid skeletal mesh"), [this]()
		{
			UPhysicsAsset* Result = UPhysicsAssetToolset::CreateFromMesh(
				TEXT("/Engine/EditorMeshes/SkeletalMesh/DefaultSkeletalMesh"), false);
			if (TestNotNull("Asset created", Result))
			{
				TestTrue("Has at least one body",
					UPhysicsAssetToolset::GetBodyNames(Result).Num() > 0);
			}
		});
	});

	// -------------------------------------------------------------------------
	// GetBodyNames
	// -------------------------------------------------------------------------
	Describe(TEXT("GetBodyNames"), [this]()
	{
		BeforeEach([this]() { SetUpInMemoryAsset(); });
		AfterEach([this]() { TearDownInMemoryAsset(); });

		It(TEXT("Returns the bone name of each body in the asset"), [this]()
		{
			const TArray<FString> Names =
				UPhysicsAssetToolset::GetBodyNames(PhysicsAsset);
			TestTrue("Contains TestBone", Names.Contains(BoneName));
			TestEqual("Count matches body setups",
				Names.Num(), PhysicsAsset->SkeletalBodySetups.Num());
		});
	});

	// -------------------------------------------------------------------------
	// GetBodyShapes
	// -------------------------------------------------------------------------
	Describe(TEXT("GetBodyShapes"), [this]()
	{
		BeforeEach([this]() { SetUpInMemoryAsset(); });
		AfterEach([this]() { TearDownInMemoryAsset(); });

		It(TEXT("Returns an empty list for a body with no shapes"), [this]()
		{
			const TArray<FPhysicsShapeInfo> Shapes =
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName);
			TestEqual("Empty", Shapes.Num(), 0);
		});

		It(TEXT("Returns an empty list for an unknown bone name"), [this]()
		{
			const TArray<FPhysicsShapeInfo> Shapes =
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, TEXT("ZZZ_Unknown"));
			TestEqual("Empty on unknown bone", Shapes.Num(), 0);
		});
	});

	// -------------------------------------------------------------------------
	// SetSphere
	// -------------------------------------------------------------------------
	Describe(TEXT("SetSphere"), [this]()
	{
		BeforeEach([this]() { SetUpInMemoryAsset(); });
		AfterEach([this]() { TearDownInMemoryAsset(); });

		It(TEXT("Adds a sphere visible via GetBodyShapes"), [this]()
		{
			UPhysicsAssetToolset::SetSphere(
				PhysicsAsset, BoneName, TEXT("s1"), FVector(1.f, 2.f, 3.f), 10.f);

			const TArray<FPhysicsShapeInfo> Shapes =
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName);
			if (TestEqual("One shape", Shapes.Num(), 1))
			{
				TestEqual("Sphere type", Shapes[0].ShapeType, EPhysicsShapeType::Sphere);
				TestEqual("Name", Shapes[0].ShapeName, FString(TEXT("s1")));
				TestEqual("Radius", Shapes[0].Radius, 10.f);
				TestEqual("Center", Shapes[0].Center, FVector(1.f, 2.f, 3.f));
			}
		});

		It(TEXT("Replaces an existing sphere that has the same name"), [this]()
		{
			UPhysicsAssetToolset::SetSphere(
				PhysicsAsset, BoneName, TEXT("s1"), FVector::ZeroVector, 10.f);
			UPhysicsAssetToolset::SetSphere(
				PhysicsAsset, BoneName, TEXT("s1"), FVector::ZeroVector, 25.f);

			const TArray<FPhysicsShapeInfo> Shapes =
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName);
			if (TestEqual("Still one shape", Shapes.Num(), 1))
			{
				TestEqual("Updated radius", Shapes[0].Radius, 25.f);
			}
		});

		It(TEXT("Replaces a shape of a different type that has the same name"), [this]()
		{
			UPhysicsAssetToolset::SetBox(
				PhysicsAsset, BoneName, TEXT("main"),
				FVector::ZeroVector, FRotator::ZeroRotator, 10.f, 10.f, 10.f);
			UPhysicsAssetToolset::SetSphere(
				PhysicsAsset, BoneName, TEXT("main"), FVector::ZeroVector, 5.f);

			const TArray<FPhysicsShapeInfo> Shapes =
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName);
			if (TestEqual("Still one shape", Shapes.Num(), 1))
			{
				TestEqual("Now a sphere", Shapes[0].ShapeType, EPhysicsShapeType::Sphere);
			}
		});

		It(TEXT("Raises an error and adds nothing for zero radius"), [this]()
		{
			UPhysicsAssetToolset::SetSphere(
				PhysicsAsset, BoneName, TEXT("s1"), FVector::ZeroVector, 0.f);
			TestEqual("No shape added",
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName).Num(), 0);
		});

		It(TEXT("Raises an error for an empty shape name"), [this]()
		{
			UPhysicsAssetToolset::SetSphere(
				PhysicsAsset, BoneName, FString(), FVector::ZeroVector, 10.f);
			TestEqual("No shape added",
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName).Num(), 0);
		});

		It(TEXT("Raises an error for an unknown bone name"), [this]()
		{
			UPhysicsAssetToolset::SetSphere(
				PhysicsAsset, TEXT("ZZZ_Unknown"), TEXT("s1"), FVector::ZeroVector, 10.f);
		});
	});

	// -------------------------------------------------------------------------
	// SetCapsule
	// -------------------------------------------------------------------------
	Describe(TEXT("SetCapsule"), [this]()
	{
		BeforeEach([this]() { SetUpInMemoryAsset(); });
		AfterEach([this]() { TearDownInMemoryAsset(); });

		It(TEXT("Adds a capsule visible via GetBodyShapes"), [this]()
		{
			UPhysicsAssetToolset::SetCapsule(
				PhysicsAsset, BoneName, TEXT("c1"),
				FVector::ZeroVector, FRotator::ZeroRotator, 5.f, 20.f);

			const TArray<FPhysicsShapeInfo> Shapes =
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName);
			if (TestEqual("One shape", Shapes.Num(), 1))
			{
				TestEqual("Capsule type", Shapes[0].ShapeType, EPhysicsShapeType::Capsule);
				TestEqual("Radius", Shapes[0].Radius, 5.f);
				TestEqual("Length", Shapes[0].Length, 20.f);
			}
		});

		It(TEXT("Raises an error for zero radius"), [this]()
		{
			UPhysicsAssetToolset::SetCapsule(
				PhysicsAsset, BoneName, TEXT("c1"),
				FVector::ZeroVector, FRotator::ZeroRotator, 0.f, 20.f);
			TestEqual("No shape added",
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName).Num(), 0);
		});

		It(TEXT("Raises an error for negative length"), [this]()
		{
			UPhysicsAssetToolset::SetCapsule(
				PhysicsAsset, BoneName, TEXT("c1"),
				FVector::ZeroVector, FRotator::ZeroRotator, 5.f, -1.f);
			TestEqual("No shape added",
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName).Num(), 0);
		});

		It(TEXT("Accepts zero length for a degenerate sphere-capped capsule"), [this]()
		{
			UPhysicsAssetToolset::SetCapsule(
				PhysicsAsset, BoneName, TEXT("c1"),
				FVector::ZeroVector, FRotator::ZeroRotator, 5.f, 0.f);
			TestEqual("Shape added",
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName).Num(), 1);
		});
	});

	// -------------------------------------------------------------------------
	// SetBox
	// -------------------------------------------------------------------------
	Describe(TEXT("SetBox"), [this]()
	{
		BeforeEach([this]() { SetUpInMemoryAsset(); });
		AfterEach([this]() { TearDownInMemoryAsset(); });

		It(TEXT("Adds a box visible via GetBodyShapes"), [this]()
		{
			UPhysicsAssetToolset::SetBox(
				PhysicsAsset, BoneName, TEXT("b1"),
				FVector::ZeroVector, FRotator::ZeroRotator, 10.f, 20.f, 30.f);

			const TArray<FPhysicsShapeInfo> Shapes =
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName);
			if (TestEqual("One shape", Shapes.Num(), 1))
			{
				TestEqual("Box type", Shapes[0].ShapeType, EPhysicsShapeType::Box);
				TestEqual("ExtentX", Shapes[0].ExtentX, 10.f);
				TestEqual("ExtentY", Shapes[0].ExtentY, 20.f);
				TestEqual("ExtentZ", Shapes[0].ExtentZ, 30.f);
			}
		});

		It(TEXT("Raises an error for zero ExtentX"), [this]()
		{
			UPhysicsAssetToolset::SetBox(
				PhysicsAsset, BoneName, TEXT("b1"),
				FVector::ZeroVector, FRotator::ZeroRotator, 0.f, 10.f, 10.f);
			TestEqual("No shape added",
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName).Num(), 0);
		});

		It(TEXT("Raises an error for zero ExtentY"), [this]()
		{
			UPhysicsAssetToolset::SetBox(
				PhysicsAsset, BoneName, TEXT("b1"),
				FVector::ZeroVector, FRotator::ZeroRotator, 10.f, 0.f, 10.f);
			TestEqual("No shape added",
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName).Num(), 0);
		});

		It(TEXT("Raises an error for zero ExtentZ"), [this]()
		{
			UPhysicsAssetToolset::SetBox(
				PhysicsAsset, BoneName, TEXT("b1"),
				FVector::ZeroVector, FRotator::ZeroRotator, 10.f, 10.f, 0.f);
			TestEqual("No shape added",
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName).Num(), 0);
		});
	});

	// -------------------------------------------------------------------------
	// RemoveShape
	// -------------------------------------------------------------------------
	Describe(TEXT("RemoveShape"), [this]()
	{
		BeforeEach([this]() { SetUpInMemoryAsset(); });
		AfterEach([this]() { TearDownInMemoryAsset(); });

		It(TEXT("Removes a sphere by name"), [this]()
		{
			UPhysicsAssetToolset::SetSphere(
				PhysicsAsset, BoneName, TEXT("s1"), FVector::ZeroVector, 10.f);
			UPhysicsAssetToolset::RemoveShape(PhysicsAsset, BoneName, TEXT("s1"));
			TestEqual("Empty after removal",
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName).Num(), 0);
		});

		It(TEXT("Removes a capsule by name"), [this]()
		{
			UPhysicsAssetToolset::SetCapsule(
				PhysicsAsset, BoneName, TEXT("c1"),
				FVector::ZeroVector, FRotator::ZeroRotator, 5.f, 10.f);
			UPhysicsAssetToolset::RemoveShape(PhysicsAsset, BoneName, TEXT("c1"));
			TestEqual("Empty after removal",
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName).Num(), 0);
		});

		It(TEXT("Removes a box by name"), [this]()
		{
			UPhysicsAssetToolset::SetBox(
				PhysicsAsset, BoneName, TEXT("b1"),
				FVector::ZeroVector, FRotator::ZeroRotator, 10.f, 10.f, 10.f);
			UPhysicsAssetToolset::RemoveShape(PhysicsAsset, BoneName, TEXT("b1"));
			TestEqual("Empty after removal",
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName).Num(), 0);
		});

		It(TEXT("Leaves other shapes intact when removing one"), [this]()
		{
			UPhysicsAssetToolset::SetSphere(
				PhysicsAsset, BoneName, TEXT("keep"), FVector::ZeroVector, 5.f);
			UPhysicsAssetToolset::SetSphere(
				PhysicsAsset, BoneName, TEXT("remove"), FVector::ZeroVector, 10.f);
			UPhysicsAssetToolset::RemoveShape(PhysicsAsset, BoneName, TEXT("remove"));

			const TArray<FPhysicsShapeInfo> Shapes =
				UPhysicsAssetToolset::GetBodyShapes(PhysicsAsset, BoneName);
			if (TestEqual("One shape remains", Shapes.Num(), 1))
			{
				TestEqual("Correct shape kept", Shapes[0].ShapeName, FString(TEXT("keep")));
			}
		});

		It(TEXT("Raises an error for a non-existent shape name"), [this]()
		{
			UPhysicsAssetToolset::RemoveShape(
				PhysicsAsset, BoneName, TEXT("ZZZ_NoSuchShape"));
		});

		It(TEXT("Raises an error for a non-existent bone name"), [this]()
		{
			UPhysicsAssetToolset::RemoveShape(
				PhysicsAsset, TEXT("ZZZ_Unknown"), TEXT("s1"));
		});
	});

	// -------------------------------------------------------------------------
	// AddBody / RemoveBody
	// -------------------------------------------------------------------------
	Describe(TEXT("AddBody"), [this]()
	{
		BeforeEach([this]() { SetUpInMemoryAsset(); });
		AfterEach([this]() { TearDownInMemoryAsset(); });

		It(TEXT("Adds a body visible via GetBodyNames"), [this]()
		{
			UPhysicsAssetToolset::AddBody(PhysicsAsset, TEXT("NewBone"));
			const TArray<FString> Names = UPhysicsAssetToolset::GetBodyNames(PhysicsAsset);
			TestTrue("NewBone present", Names.Contains(TEXT("NewBone")));
		});

		It(TEXT("Raises an error for an empty bone name"), [this]()
		{
			const int32 CountBefore = UPhysicsAssetToolset::GetBodyNames(PhysicsAsset).Num();
			UPhysicsAssetToolset::AddBody(PhysicsAsset, FString());
			TestEqual("Count unchanged",
				UPhysicsAssetToolset::GetBodyNames(PhysicsAsset).Num(), CountBefore);
		});

		It(TEXT("Raises an error when the bone already has a body"), [this]()
		{
			const int32 CountBefore = UPhysicsAssetToolset::GetBodyNames(PhysicsAsset).Num();
			UPhysicsAssetToolset::AddBody(PhysicsAsset, BoneName);
			TestEqual("Count unchanged",
				UPhysicsAssetToolset::GetBodyNames(PhysicsAsset).Num(), CountBefore);
		});
	});

	Describe(TEXT("RemoveBody"), [this]()
	{
		BeforeEach([this]() { SetUpInMemoryAssetWithTwoBodies(); });
		AfterEach([this]() { TearDownInMemoryAsset(); });

		It(TEXT("Removes a body by bone name"), [this]()
		{
			UPhysicsAssetToolset::RemoveBody(PhysicsAsset, BoneName);
			const TArray<FString> Names = UPhysicsAssetToolset::GetBodyNames(PhysicsAsset);
			TestFalse("No longer present", Names.Contains(BoneName));
		});

		It(TEXT("Also removes constraints that reference the body"), [this]()
		{
			UPhysicsAssetToolset::AddConstraint(PhysicsAsset, BoneName, BoneName2);
			UPhysicsAssetToolset::RemoveBody(PhysicsAsset, BoneName);
			TestEqual("No constraints remain",
				UPhysicsAssetToolset::GetConstraints(PhysicsAsset).Num(), 0);
		});

		It(TEXT("Raises an error for a non-existent bone name"), [this]()
		{
			const int32 CountBefore = UPhysicsAssetToolset::GetBodyNames(PhysicsAsset).Num();
			UPhysicsAssetToolset::RemoveBody(PhysicsAsset, TEXT("ZZZ_Unknown"));
			TestEqual("Count unchanged",
				UPhysicsAssetToolset::GetBodyNames(PhysicsAsset).Num(), CountBefore);
		});
	});

	// -------------------------------------------------------------------------
	// GetBodyPhysicsMode / SetBodyPhysicsMode
	// -------------------------------------------------------------------------
	Describe(TEXT("GetBodyPhysicsMode and SetBodyPhysicsMode"), [this]()
	{
		BeforeEach([this]() { SetUpInMemoryAsset(); });
		AfterEach([this]() { TearDownInMemoryAsset(); });

		It(TEXT("Default mode is Default"), [this]()
		{
			TestEqual("Default",
				UPhysicsAssetToolset::GetBodyPhysicsMode(PhysicsAsset, BoneName),
				EBodyPhysicsMode::Default);
		});

		It(TEXT("Can set and read back Kinematic"), [this]()
		{
			UPhysicsAssetToolset::SetBodyPhysicsMode(
				PhysicsAsset, BoneName, EBodyPhysicsMode::Kinematic);
			TestEqual("Kinematic",
				UPhysicsAssetToolset::GetBodyPhysicsMode(PhysicsAsset, BoneName),
				EBodyPhysicsMode::Kinematic);
		});

		It(TEXT("Can set and read back Simulated"), [this]()
		{
			UPhysicsAssetToolset::SetBodyPhysicsMode(
				PhysicsAsset, BoneName, EBodyPhysicsMode::Simulated);
			TestEqual("Simulated",
				UPhysicsAssetToolset::GetBodyPhysicsMode(PhysicsAsset, BoneName),
				EBodyPhysicsMode::Simulated);
		});

		It(TEXT("Raises an error for an unknown bone name"), [this]()
		{
			UPhysicsAssetToolset::SetBodyPhysicsMode(
				PhysicsAsset, TEXT("ZZZ_Unknown"), EBodyPhysicsMode::Simulated);
		});
	});

	// -------------------------------------------------------------------------
	// GetBodyMassScale / SetBodyMassScale
	// -------------------------------------------------------------------------
	Describe(TEXT("GetBodyMassScale and SetBodyMassScale"), [this]()
	{
		BeforeEach([this]() { SetUpInMemoryAsset(); });
		AfterEach([this]() { TearDownInMemoryAsset(); });

		It(TEXT("Can set and read back a mass scale"), [this]()
		{
			UPhysicsAssetToolset::SetBodyMassScale(PhysicsAsset, BoneName, 2.5f);
			TestEqual("MassScale",
				UPhysicsAssetToolset::GetBodyMassScale(PhysicsAsset, BoneName), 2.5f);
		});

		It(TEXT("Raises an error for zero mass scale"), [this]()
		{
			UPhysicsAssetToolset::SetBodyMassScale(PhysicsAsset, BoneName, 0.f);
		});

		It(TEXT("Raises an error for negative mass scale"), [this]()
		{
			UPhysicsAssetToolset::SetBodyMassScale(PhysicsAsset, BoneName, -1.f);
		});

		It(TEXT("Raises an error for an unknown bone name"), [this]()
		{
			UPhysicsAssetToolset::SetBodyMassScale(
				PhysicsAsset, TEXT("ZZZ_Unknown"), 1.f);
		});
	});

	// -------------------------------------------------------------------------
	// GetConstraints / AddConstraint / SetConstraintLimits / RemoveConstraint
	// -------------------------------------------------------------------------
	Describe(TEXT("GetConstraints"), [this]()
	{
		BeforeEach([this]() { SetUpInMemoryAssetWithTwoBodies(); });
		AfterEach([this]() { TearDownInMemoryAsset(); });

		It(TEXT("Returns an empty list when there are no constraints"), [this]()
		{
			TestEqual("Empty",
				UPhysicsAssetToolset::GetConstraints(PhysicsAsset).Num(), 0);
		});

		It(TEXT("Returns one entry per constraint"), [this]()
		{
			UPhysicsAssetToolset::AddConstraint(PhysicsAsset, BoneName, BoneName2);
			TestEqual("One constraint",
				UPhysicsAssetToolset::GetConstraints(PhysicsAsset).Num(), 1);
		});
	});

	Describe(TEXT("AddConstraint"), [this]()
	{
		BeforeEach([this]() { SetUpInMemoryAssetWithTwoBodies(); });
		AfterEach([this]() { TearDownInMemoryAsset(); });

		It(TEXT("Adds a constraint visible via GetConstraints"), [this]()
		{
			UPhysicsAssetToolset::AddConstraint(PhysicsAsset, BoneName, BoneName2);
			const TArray<FPhysicsConstraintInfo> Constraints =
				UPhysicsAssetToolset::GetConstraints(PhysicsAsset);
			if (TestEqual("One constraint", Constraints.Num(), 1))
			{
				TestEqual("Bone1", Constraints[0].Bone1Name, BoneName);
				TestEqual("Bone2", Constraints[0].Bone2Name, BoneName2);
			}
		});

		It(TEXT("Raises an error when a constraint already exists"), [this]()
		{
			UPhysicsAssetToolset::AddConstraint(PhysicsAsset, BoneName, BoneName2);
			UPhysicsAssetToolset::AddConstraint(PhysicsAsset, BoneName, BoneName2);
			TestEqual("Still one constraint",
				UPhysicsAssetToolset::GetConstraints(PhysicsAsset).Num(), 1);
		});

		It(TEXT("Raises an error for empty bone names"), [this]()
		{
			UPhysicsAssetToolset::AddConstraint(PhysicsAsset, FString(), BoneName2);
			UPhysicsAssetToolset::AddConstraint(PhysicsAsset, BoneName, FString());
			TestEqual("No constraints added",
				UPhysicsAssetToolset::GetConstraints(PhysicsAsset).Num(), 0);
		});

		It(TEXT("Raises an error when a bone has no body"), [this]()
		{
			UPhysicsAssetToolset::AddConstraint(
				PhysicsAsset, BoneName, TEXT("ZZZ_Unknown"));
			TestEqual("No constraint added",
				UPhysicsAssetToolset::GetConstraints(PhysicsAsset).Num(), 0);
		});
	});

	Describe(TEXT("SetConstraintLimits"), [this]()
	{
		BeforeEach([this]()
		{
			SetUpInMemoryAssetWithTwoBodies();
			UPhysicsAssetToolset::AddConstraint(PhysicsAsset, BoneName, BoneName2);
		});
		AfterEach([this]() { TearDownInMemoryAsset(); });

		It(TEXT("Sets angular limits and reads them back via GetConstraints"), [this]()
		{
			FPhysicsConstraintInfo Info;
			Info.Bone1Name = BoneName;
			Info.Bone2Name = BoneName2;
			Info.Swing1Motion = EConstraintMotion::Limited;
			Info.Swing1LimitDegrees = 30.f;
			Info.Swing2Motion = EConstraintMotion::Locked;
			Info.Swing2LimitDegrees = 0.f;
			Info.TwistMotion = EConstraintMotion::Limited;
			Info.TwistLimitDegrees = 45.f;

			UPhysicsAssetToolset::SetConstraintLimits(PhysicsAsset, Info);

			const TArray<FPhysicsConstraintInfo> Constraints =
				UPhysicsAssetToolset::GetConstraints(PhysicsAsset);
			if (TestEqual("One constraint", Constraints.Num(), 1))
			{
				TestEqual("Swing1Motion", Constraints[0].Swing1Motion,
					EConstraintMotion::Limited);
				TestEqual("Swing1Limit", Constraints[0].Swing1LimitDegrees, 30.f);
				TestEqual("Swing2Motion", Constraints[0].Swing2Motion,
					EConstraintMotion::Locked);
				TestEqual("TwistMotion", Constraints[0].TwistMotion,
					EConstraintMotion::Limited);
				TestEqual("TwistLimit", Constraints[0].TwistLimitDegrees, 45.f);
			}
		});

		It(TEXT("Raises an error when the constraint does not exist"), [this]()
		{
			FPhysicsConstraintInfo Info;
			Info.Bone1Name = TEXT("ZZZ_A");
			Info.Bone2Name = TEXT("ZZZ_B");
			UPhysicsAssetToolset::SetConstraintLimits(PhysicsAsset, Info);
		});
	});

	Describe(TEXT("RemoveConstraint"), [this]()
	{
		BeforeEach([this]()
		{
			SetUpInMemoryAssetWithTwoBodies();
			UPhysicsAssetToolset::AddConstraint(PhysicsAsset, BoneName, BoneName2);
		});
		AfterEach([this]() { TearDownInMemoryAsset(); });

		It(TEXT("Removes an existing constraint"), [this]()
		{
			UPhysicsAssetToolset::RemoveConstraint(PhysicsAsset, BoneName, BoneName2);
			TestEqual("Empty after removal",
				UPhysicsAssetToolset::GetConstraints(PhysicsAsset).Num(), 0);
		});

		It(TEXT("Raises an error for a non-existent constraint"), [this]()
		{
			UPhysicsAssetToolset::RemoveConstraint(
				PhysicsAsset, TEXT("ZZZ_A"), TEXT("ZZZ_B"));
			TestEqual("Existing constraint unaffected",
				UPhysicsAssetToolset::GetConstraints(PhysicsAsset).Num(), 1);
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
